{
  "MajorVersion": 5,
  "MinorVersion": 8,
  "PatchVersion": 1,
  "Changelist": 56057345,
  "CompatibleChangelist": 55116800,
  "IsLicenseeVersion": 0,
  "IsPromotedBuild": 0,
  "BranchName": "++UE5+Release-5.8"
}